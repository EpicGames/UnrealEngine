// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubtitleComponentTypes.h"

#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

bool FMovieSceneSubtitleComponentTypes::GMovieSceneSubtitlesComponentTypesDestroyed = false;
TUniquePtr<FMovieSceneSubtitleComponentTypes> FMovieSceneSubtitleComponentTypes::GMovieSceneSubtitlesComponentTypes;


FMovieSceneSubtitleComponentTypes* FMovieSceneSubtitleComponentTypes::Get()
{
	if (!GMovieSceneSubtitlesComponentTypes.IsValid())
	{
		check(!GMovieSceneSubtitlesComponentTypesDestroyed);
		GMovieSceneSubtitlesComponentTypes.Reset(new FMovieSceneSubtitleComponentTypes);
	}
	return GMovieSceneSubtitlesComponentTypes.Get();
}

void FMovieSceneSubtitleComponentTypes::Destroy()
{
	GMovieSceneSubtitlesComponentTypes.Reset();
	GMovieSceneSubtitlesComponentTypesDestroyed = true;
}

FMovieSceneSubtitleComponentTypes::FMovieSceneSubtitleComponentTypes()
{
	using namespace UE::MovieScene;
	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();
	check(ComponentRegistry);
	
	ComponentRegistry->NewComponentType(&SubtitleData, TEXT("SubtitleData Component"));
}
