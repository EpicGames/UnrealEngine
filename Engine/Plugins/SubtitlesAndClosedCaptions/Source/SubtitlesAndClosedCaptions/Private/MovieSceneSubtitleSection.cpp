// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubtitleSection.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSubtitleComponentTypes.h"
#include "SubtitleDataComponent.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSubtitleSection)

using namespace UE::MovieScene;

UMovieSceneSubtitleSection::UMovieSceneSubtitleSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TOptional<TRange<FFrameNumber>> UMovieSceneSubtitleSection::GetAutoSizeRange() const
{
	if (!Subtitle)
	{
		return TRange<FFrameNumber>();
	}

	// Find the maximum duration of all the subtitles in the UserData, including their delayed start offsets.
	const FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	const FFrameTime DurationToUse = Subtitle->GetMaximumDuration() * FrameRate;
	return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + DurationToUse.FrameNumber);
}

bool UMovieSceneSubtitleSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	// check preconditions
	check(OutFieldBuilder);

	OutFieldBuilder->AddPersistentEntity(EffectiveRange, this);
	return true;
}

void UMovieSceneSubtitleSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	// check preconditions
	check(OutImportedEntity);

	if (!Subtitle)
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	check(BuiltInComponents);

	FMovieSceneSubtitleComponentTypes* SubtitleComponents = FMovieSceneSubtitleComponentTypes::Get();
	check(SubtitleComponents);

	const FGuid ObjectBindingID = Params.GetObjectBindingID();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
		.Add(SubtitleComponents->SubtitleData, FSubtitleDataComponent{ this, EMovieScenePlayerStatus::Type::Stopped }));
}
