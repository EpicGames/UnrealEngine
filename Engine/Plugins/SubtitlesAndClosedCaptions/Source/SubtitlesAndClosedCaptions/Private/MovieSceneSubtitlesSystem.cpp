// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubtitlesSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"

#include "MovieSceneSubtitleComponentTypes.h"
#include "MovieSceneSubtitleSection.h"

#include "SubtitlesAndClosedCaptionsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSubtitlesSystem)

using namespace UE::MovieScene;


UMovieSceneSubtitlesSystem::UMovieSceneSubtitlesSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	FMovieSceneSubtitleComponentTypes* SubtitlesComponents = FMovieSceneSubtitleComponentTypes::Get();
	check(SubtitlesComponents);

	RelevantComponent = SubtitlesComponents->SubtitleData;

	Phase = ESystemPhase::Scheduling;
}

UMovieSceneSubtitlesSystem::~UMovieSceneSubtitlesSystem()
{
}

void UMovieSceneSubtitlesSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	check(BuiltInComponents);

	const FMovieSceneSubtitleComponentTypes* SubtitleComponents = FMovieSceneSubtitleComponentTypes::Get();
	check(SubtitleComponents);

	check(Linker);
	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	check(InstanceRegistry);

	FTaskID EvaluateTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Write(SubtitleComponents->SubtitleData)
		.SetDesiredThread(ENamedThreads::GameThread)
		.Schedule_PerAllocation<FEvaluateSubtitles>(&Linker->EntityManager, TaskScheduler, *InstanceRegistry);
}

void UMovieSceneSubtitlesSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	check(BuiltInComponents);

	const FMovieSceneSubtitleComponentTypes* SubtitleComponents = FMovieSceneSubtitleComponentTypes::Get();
	check(SubtitleComponents);

	check(Linker);
	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	check(InstanceRegistry);

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Write(SubtitleComponents->SubtitleData)
		.SetDesiredThread(ENamedThreads::GameThread)
		.template Dispatch_PerAllocation<FEvaluateSubtitles>(&Linker->EntityManager, InPrerequisites, &Subsequents, *InstanceRegistry);


}
