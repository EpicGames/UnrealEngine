// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubtitlesTrack.h"

#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSubtitleSection.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSubtitlesTrack)

#define LOCTEXT_NAMESPACE "MovieSceneSubtitlesTrack"

constexpr float DefaultSubtitleSectionDuration = 3.f;

UMovieSceneSubtitlesTrack::UMovieSceneSubtitlesTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMovieSceneSection* UMovieSceneSubtitlesTrack::AddNewSubtitle(const USubtitleAssetUserData& Subtitle, FFrameNumber InStartTime)
{
	return AddNewSubtitleOnRow(Subtitle, InStartTime, INDEX_NONE);
}

UMovieSceneSection* UMovieSceneSubtitlesTrack::AddNewSubtitleOnRow(const USubtitleAssetUserData& Subtitle, FFrameNumber InStartTime, int32 InRowIndex)
{
	UMovieSceneSubtitleSection* NewSection = Cast<UMovieSceneSubtitleSection>(CreateNewSection());

	const FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	const float SubtitleDuration = Subtitle.GetMaximumDuration();

	// Find the maximum duration of all the subtitles in the UserData, including their delayed start offsets.
	const bool UseDefaultDuration = Subtitle.Subtitles.IsEmpty();
	// Use the default duration if the subtitles array is empty.
	// If it's not empty, the subtitles themselves have a minimum duration and therefore have a non-zero length.
	const FFrameTime DurationToUse = UseDefaultDuration ? (DefaultSubtitleSectionDuration * FrameRate) : (SubtitleDuration * FrameRate);

	NewSection->InitialPlacementOnRow(SubtitleSections, InStartTime, DurationToUse.FrameNumber.Value, InRowIndex);
	NewSection->SetSubtitle(Subtitle);

	// If the duration was 0-length, turn orange to make it clear that the duration was defaulted.
	if (UseDefaultDuration)
	{
		NewSection->SetColorTint(FColor::Orange);
	}

	SubtitleSections.Add(NewSection);

	return NewSection;
}

bool UMovieSceneSubtitlesTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneSubtitleSection::StaticClass();
}

void UMovieSceneSubtitlesTrack::RemoveAllAnimationData()
{
	return SubtitleSections.Empty();
}

bool UMovieSceneSubtitlesTrack::HasSection(const UMovieSceneSection& Section) const
{
	return SubtitleSections.Contains(&Section);
}

void UMovieSceneSubtitlesTrack::AddSection(UMovieSceneSection& Section)
{
	SubtitleSections.Add(&Section);
}

void UMovieSceneSubtitlesTrack::RemoveSection(UMovieSceneSection& Section)
{
	SubtitleSections.Remove(&Section);
}

void UMovieSceneSubtitlesTrack::RemoveSectionAt(int32 SectionIndex)
{
	SubtitleSections.RemoveAt(SectionIndex);
}

bool UMovieSceneSubtitlesTrack::IsEmpty() const
{
	return SubtitleSections.IsEmpty();
}

const TArray<UMovieSceneSection*>& UMovieSceneSubtitlesTrack::GetAllSections() const
{
	return SubtitleSections;
}

bool UMovieSceneSubtitlesTrack::SupportsMultipleRows() const
{
	return true;
}

UMovieSceneSection* UMovieSceneSubtitlesTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSubtitleSection>(this, NAME_None, RF_Transactional);
}

#undef LOCTEXT_NAMESPACE
