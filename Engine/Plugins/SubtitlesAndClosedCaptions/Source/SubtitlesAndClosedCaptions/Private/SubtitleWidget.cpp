// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleWidget.h"
#include "Components/Border.h"
#include "Components/TextBlock.h"
#include "Logging/StructuredLog.h"
#include "SubtitlesAndClosedCaptionsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitleWidget)

void USubtitleWidget::NativeConstruct()
{
	Super::NativeConstruct();

	StopDisplayingAllSubtitles();
}

void USubtitleWidget::StartDisplayingSubtitle(const FSubtitleAssetData& Subtitle)
{
	// Pick which UextBlock is relevant by category....
	UTextBlock* TextToModify = nullptr;
	UBorder* BorderToModify = nullptr;

	const ESubtitleType SubtitleType = Subtitle.SubtitleType;

	switch (SubtitleType)
	{
	default:
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "An unrecognized subtitle type was queued for display. Using the standard Subtitle text block as a fallback.");
		// Fallthrough, no break.
	}

	case ESubtitleType::Subtitle:
	{
		TextToModify = DialogSubtitleBlock;
		BorderToModify = DialogBorder;
		break;
	}

	case ESubtitleType::ClosedCaption:
	{
		TextToModify = CaptionSubtitleBlock;
		BorderToModify = CaptionBorder;
		break;
	}

	case ESubtitleType::AudioDescription:
	{
		TextToModify = DescriptionSubtitleBlock;
		BorderToModify = DescriptionBorder;
		break;
	}

	}

	// ...Then modify and display it
	if (IsValid(TextToModify))
	{
		TextToModify->SetText(Subtitle.Text);
		TextToModify->SetVisibility(ESlateVisibility::HitTestInvisible);

		// Only hide/display border if the corresponding text is also valid.
		if (IsValid(BorderToModify))
		{
			BorderToModify->SetVisibility(ESlateVisibility::HitTestInvisible);
		}

		// Track the currently-displayed subtitle for comparison when removing later
		CurrentlyDisplayedSubtitles[SubtitleType] = Subtitle;
	}
}

void USubtitleWidget::StopDisplayingSubtitle(const FSubtitleAssetData& Subtitle)
{
	UTextBlock* TextToModify = nullptr;
	UBorder* BorderToModify = nullptr;

	// Confirm that the removed subtitle is actually being displayed before trying to stop it.
	const ESubtitleType SubtitleType = Subtitle.SubtitleType;
	if (CurrentlyDisplayedSubtitles[SubtitleType].IsSet() && CurrentlyDisplayedSubtitles[SubtitleType] == Subtitle)
	{
		switch (SubtitleType)
		{
		default: // Fallthrough, no break (see ::StartDisplayingSubtitle)
		case ESubtitleType::Subtitle:
		{
			TextToModify = DialogSubtitleBlock;
			BorderToModify = DialogBorder;
			break;
		}

		case ESubtitleType::ClosedCaption:
		{
			TextToModify = CaptionSubtitleBlock;
			BorderToModify = CaptionBorder;
			break;
		}

		case ESubtitleType::AudioDescription:
		{
			TextToModify = DescriptionSubtitleBlock;
			BorderToModify = DescriptionBorder;
			break;
		}

		}

		if (IsValid(TextToModify))
		{
			TextToModify->SetVisibility(ESlateVisibility::Collapsed);

			if (IsValid(BorderToModify))
			{
				BorderToModify->SetVisibility(ESlateVisibility::Collapsed);
			}
		}
	}
}

void USubtitleWidget::StopDisplayingAllSubtitles()
{
	// Ensure the text blocks don't start visible.
	if (IsValid(DialogSubtitleBlock))
	{
		DialogSubtitleBlock->SetVisibility(ESlateVisibility::Collapsed);
	}

	if (IsValid(CaptionSubtitleBlock))
	{
		CaptionSubtitleBlock->SetVisibility(ESlateVisibility::Collapsed);
	}

	if (IsValid(DescriptionSubtitleBlock))
	{
		DescriptionSubtitleBlock->SetVisibility(ESlateVisibility::Collapsed);
	}

	// Borders should also not be visible on startup.
	if (IsValid(DialogBorder))
	{
		DialogBorder->SetVisibility(ESlateVisibility::Collapsed);
	}
	if (IsValid(CaptionBorder))
	{
		CaptionBorder->SetVisibility(ESlateVisibility::Collapsed);
	}
	if (IsValid(DescriptionBorder))
	{
		DescriptionBorder->SetVisibility(ESlateVisibility::Collapsed);
	}
}

USubtitleWidget::USubtitleWidget(const FObjectInitializer& ObjectInitializer)
	: UUserWidget(ObjectInitializer)
{

	CurrentlyDisplayedSubtitles.Add({ ESubtitleType::Subtitle,			TOptional<FSubtitleAssetData>() });
	CurrentlyDisplayedSubtitles.Add({ ESubtitleType::AudioDescription,	TOptional<FSubtitleAssetData>() });
	CurrentlyDisplayedSubtitles.Add({ ESubtitleType::ClosedCaption,		TOptional<FSubtitleAssetData>() });
}
