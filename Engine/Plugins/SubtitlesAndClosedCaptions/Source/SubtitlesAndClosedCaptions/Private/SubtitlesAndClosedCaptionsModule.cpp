// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesAndClosedCaptionsModule.h"

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

DEFINE_LOG_CATEGORY(LogSubtitlesAndClosedCaptions);

void FSubtitlesAndClosedCaptionsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(ISubtitlesAndClosedCaptionsModule::GetModularFeatureName(), this);
}

void FSubtitlesAndClosedCaptionsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(ISubtitlesAndClosedCaptionsModule::GetModularFeatureName(), this);
}

TSubclassOf<UAssetUserData> FSubtitlesAndClosedCaptionsModule::GetAssetUserDataClass() const
{
	return USubtitleAssetUserData::StaticClass();
}

IMPLEMENT_MODULE(FSubtitlesAndClosedCaptionsModule, SubtitlesAndClosedCaptions);
