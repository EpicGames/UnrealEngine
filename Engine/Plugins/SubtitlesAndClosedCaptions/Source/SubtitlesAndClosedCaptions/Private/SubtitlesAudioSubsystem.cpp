// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesAudioSubsystem.h"

#include "ActiveSound.h"
#include "Sound/DialogueSoundWaveProxy.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitlesAudioSubsystem)

void USubtitlesAudioSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	USoundNodeDialoguePlayer::OnDialogueSoundWaveProxyParsed.AddUObject(this, &USubtitlesAudioSubsystem::OnDialogueSoundWaveProxyParsed);
}

void USubtitlesAudioSubsystem::Deinitialize()
{
	USoundNodeDialoguePlayer::OnDialogueSoundWaveProxyParsed.RemoveAll(this);
	Super::Deinitialize();
}

void USubtitlesAudioSubsystem::NotifyActiveSoundCreated(FActiveSound& ActiveSound)
{
	USoundBase* Sound = ActiveSound.GetSound();

	if (IsValid(Sound))
	{
		if (Sound->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
		{
			const USubtitleAssetUserData* SubtitleAsset = CastChecked<USubtitleAssetUserData>(Sound->GetAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()));
			for (const FSubtitleAssetData& Subtitle : SubtitleAsset->Subtitles)
			{
				const bool bUseDurationProperty = (Subtitle.SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
				FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(FQueueSubtitleParameters{ Subtitle }, bUseDurationProperty ? ESubtitleTiming::InternallyTimed : ESubtitleTiming::ExternallyTimed);
			}
		}

		// Handle DialogueWaves played directly here.
		// DialogueWaves inside SoundCues are handled via the OnDialogueSoundWaveProxyParsed delegate (see SoundNodeDialoguePlayer::ParseNodes).
		static const IConsoleVariable* CVarUseNewSubtitles = IConsoleManager::Get().FindConsoleVariable(TEXT("au.UseNewSubtitles"));

		if (CVarUseNewSubtitles && CVarUseNewSubtitles->GetBool() && ActiveSound.bHandleSubtitles)
		{
			const UDialogueSoundWaveProxy* DialogueWave = Cast<UDialogueSoundWaveProxy>(Sound);
			if (IsValid(DialogueWave))
			{
				QueueSubtitleFromDialogueWaveProxy(DialogueWave, ActiveSound);
			}
		}
	}
}

void USubtitlesAudioSubsystem::NotifyActiveSoundDeleting(const FActiveSound& ActiveSound)
{
	USoundBase* Sound = ActiveSound.GetSound();

	if (IsValid(Sound))
	{
		if (Sound->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
		{
			const USubtitleAssetUserData* SubtitleAsset = CastChecked<USubtitleAssetUserData>(Sound->GetAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()));
			for (const FSubtitleAssetData& Subtitle : SubtitleAsset->Subtitles)
			{
				const bool bUseDurationProperty = (Subtitle.SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
				// If not using sound duration, subtitles continue to play instead of stopping:
				// Their duration is completely decoupled from the sound's.

				// ...but if Using the sound's duration: Stop All subtitles.
				if (!bUseDurationProperty)
				{
					FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(Subtitle);
				}
			}
		}

		static const IConsoleVariable* CVarUseNewSubtitles = IConsoleManager::Get().FindConsoleVariable(TEXT("au.UseNewSubtitles"));

		if (CVarUseNewSubtitles && CVarUseNewSubtitles->GetBool() && ActiveSound.bHandleSubtitles)
		{
			const UDialogueSoundWaveProxy* DialogueWave = Cast<UDialogueSoundWaveProxy>(Sound);
			if (IsValid(DialogueWave))
			{
				StopSubtitleFromDialogueWaveProxy(DialogueWave, ActiveSound);
			}
			else if (TArray<FSubtitleAssetData>* Subtitles = QueuedSubtitlesByPlayOrder.Find(ActiveSound.GetPlayOrder()))
			{
				for (const FSubtitleAssetData& Subtitle : *Subtitles)
				{
					FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(Subtitle);
				}
				QueuedSubtitlesByPlayOrder.Remove(ActiveSound.GetPlayOrder());
			}
		}
	}
}

void USubtitlesAudioSubsystem::OnDialogueSoundWaveProxyParsed(const FActiveSound& ActiveSound, const UDialogueSoundWaveProxy* Proxy)
{
	if (ActiveSound.bHandleSubtitles)
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesAudioSubsystem::QueueSubtitleFromParsedSoundWaveProxy"),

			[WeakThis = TWeakObjectPtr<USubtitlesAudioSubsystem>(this),
			Text = Proxy->GetSubtitleText(), Priority = ActiveSound.SubtitlePriority, Duration = Proxy->GetDuration(),
			StartOffset = ActiveSound.RequestedStartTime, PlayOrder = ActiveSound.GetPlayOrder()]()
			{
				if (TStrongObjectPtr<USubtitlesAudioSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->QueueSubtitleFromParsedSoundWaveProxy(Text, Priority, Duration, StartOffset, PlayOrder);
				}
			}
		);
	}
}

void USubtitlesAudioSubsystem::QueueSubtitleFromParsedSoundWaveProxy(const FText Text, const float Priority, const float Duration, const float StartOffset, const uint32 PlayOrder)
{
	// Called from the audio thread, executed on game thread: fired synchronously from Broadcast inside SoundNodeDialoguePlayer::ParseNodes.
	static const IConsoleVariable* CVarUseNewSubtitles = IConsoleManager::Get().FindConsoleVariable(TEXT("au.UseNewSubtitles"));

	if (!CVarUseNewSubtitles || !CVarUseNewSubtitles->GetBool())
	{
		return;
	}

	FSubtitleAssetData SubtitleCue;
	SubtitleCue.Text = Text;
	SubtitleCue.Priority = Priority;
	SubtitleCue.Duration = Duration;
	SubtitleCue.StartOffset = StartOffset;

	QueuedSubtitlesByPlayOrder.FindOrAdd(PlayOrder).Add(SubtitleCue);

	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(FQueueSubtitleParameters{ SubtitleCue }, ESubtitleTiming::InternallyTimed);
}

void USubtitlesAudioSubsystem::QueueSubtitleFromDialogueWaveProxy(const UDialogueSoundWaveProxy* DialogueWaveProxy, const FActiveSound& ActiveSound)
{
	check(ActiveSound.bHandleSubtitles); // Function should only be called if the Active Sound actually has subtitles it needs queued.
	FSubtitleAssetData SubtitleCue;
	SubtitleCue.Text = DialogueWaveProxy->GetSubtitleText();
	SubtitleCue.Priority = ActiveSound.SubtitlePriority;
	SubtitleCue.Duration = DialogueWaveProxy->GetDuration();
	SubtitleCue.StartOffset = ActiveSound.RequestedStartTime;

	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(FQueueSubtitleParameters{ SubtitleCue }, ESubtitleTiming::InternallyTimed);
}

void USubtitlesAudioSubsystem::StopSubtitleFromDialogueWaveProxy(const UDialogueSoundWaveProxy* DialogueWaveProxy, const FActiveSound& ActiveSound)
{
	check(ActiveSound.bHandleSubtitles); // Function should only be called if the Active Sound actually has subtitles it needs queued.
	FSubtitleAssetData SubtitleCue;
	SubtitleCue.Text = DialogueWaveProxy->GetSubtitleText();
	SubtitleCue.Priority = ActiveSound.SubtitlePriority;
	SubtitleCue.Duration = DialogueWaveProxy->GetDuration();
	SubtitleCue.StartOffset = ActiveSound.RequestedStartTime;

	FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(SubtitleCue);
}
