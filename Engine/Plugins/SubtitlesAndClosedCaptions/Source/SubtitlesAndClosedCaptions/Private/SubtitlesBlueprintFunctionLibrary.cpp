// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesBlueprintFunctionLibrary.h"

#include "Engine/Engine.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "SubtitlesSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitlesBlueprintFunctionLibrary)

void USubtitlesBlueprintFunctionLibrary::QueueSubtitlesFromAsset(TSoftObjectPtr<USubtitleAssetUserData> SubtitleAsset)
{
	if (SubtitleAsset.IsValid())
	{
		for (const FSubtitleAssetData& Subtitle : SubtitleAsset->Subtitles)
		{
			FQueueSubtitleParameters Params{ Subtitle };

			// Force internal timing: external timing requires the caller to manually start and stop each subtitle in the asset, defeating this function's purpose.
			// For manual timing control on a per-subtitle basis, QueueSubtitle exists.
			FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(Params, ESubtitleTiming::InternallyTimed);
		}
	}
}

void USubtitlesBlueprintFunctionLibrary::StopSubtitlesInAsset(TSoftObjectPtr<USubtitleAssetUserData> SubtitleAsset)
{
	if (SubtitleAsset.IsValid())
	{
		for (const FSubtitleAssetData& Subtitle : SubtitleAsset->Subtitles)
		{
			FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(Subtitle);
		}
	}
}

// Old deprecated version of QueueSingleSubtitle; use that instead.
void USubtitlesBlueprintFunctionLibrary::QueueSubtitle(const FSubtitleAssetData& Subtitle, const ESubtitleTiming Timing)
{
	FQueueSubtitleParameters Params{ Subtitle };
	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(Params, Timing);
}

void USubtitlesBlueprintFunctionLibrary::QueueSingleSubtitle(
	const FText Text, const float Duration, const float StartOffset, const float Priority, const ESubtitleType SubtitleType, const ESubtitleTiming Timing)
{
	FSubtitleAssetData Subtitle{
		.Text = Text,
		.SubtitleDurationType = ESubtitleDurationType::UseDurationProperty,	// There is no sound to synchronize to, so use the provided duration.
		.Duration = Duration,
		.StartOffset = StartOffset,
		.Priority = Priority,
		.SubtitleType = SubtitleType,
	};
	FQueueSubtitleParameters Params{ Subtitle };
	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(Params, Timing);
}

bool USubtitlesBlueprintFunctionLibrary::IsSubtitleActive(const FSubtitleAssetData& Subtitle)
{
	if (!FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.IsBound())
	{
		return false;
	}
	return FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.Execute(Subtitle);
}

void USubtitlesBlueprintFunctionLibrary::StopSubtitle(const FSubtitleAssetData& Subtitle)
{
	FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(Subtitle);
}

void USubtitlesBlueprintFunctionLibrary::StopAllSubtitles()
{
	FSubtitlesAndClosedCaptionsDelegates::StopAllSubtitles.ExecuteIfBound();
}

void USubtitlesBlueprintFunctionLibrary::ReplaceSubtitleWidget(const TSoftClassPtr<USubtitleWidget>& NewWidgetAsset)
{
	// ReplaceSubtitleWidget doesn't have a corresponding delegate call (no need to expose it to the engine), so we need to get the subsystem manually as a result.
	if (GEngine == nullptr)
	{
		return;
	}

	// Get the subtitles subsystem from the visible world.
	if (const UGameViewportClient* Viewport = GEngine->GameViewport)
	{
		const FWorldContext& WorldContext = GEngine->GetWorldContextFromGameViewportChecked(Viewport);
		UWorld* World = WorldContext.World();

		if (IsValid(World))
		{
			// ReplaceWidget is non-const, so no const pointers here.
			UWorldSubsystem* WorldSubsystem = USubsystemBlueprintLibrary::GetWorldSubsystem(World, USubtitlesSubsystem::StaticClass());
			USubtitlesSubsystem* SubtitlesSubsystem = Cast<USubtitlesSubsystem>(WorldSubsystem);

			if (IsValid(SubtitlesSubsystem))
			{
				SubtitlesSubsystem->ReplaceWidget(NewWidgetAsset);
			}
		}
	}
}
