// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubtitlesSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitlesSettings)

#define LOCTEXT_NAMESPACE "SubtitlesSettings"

USubtitlesSettings::USubtitlesSettings()
{
	WidgetToUseAsSubclass = SubtitleWidgetToUse.Get();
	DefaultWidgetToUseAsSubclass = SubtitleWidgetToUseDefault.Get();
}

#undef LOCTEXT_NAMESPACE
