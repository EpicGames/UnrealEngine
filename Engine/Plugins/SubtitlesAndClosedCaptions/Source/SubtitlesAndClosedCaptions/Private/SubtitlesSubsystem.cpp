// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesSubsystem.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/Ticker.h"
#include "Engine/Font.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/SoftObjectPath.h"
#include "Logging/StructuredLog.h"
#include "Math/UnrealMathUtility.h"
#include "Sound/SoundWave.h"
#include "Styling/CoreStyle.h"
#include "SubtitlesAndClosedCaptionsModule.h"
#include "SubtitlesSettings.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "TimerManager.h"
#include "UnrealClient.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "UObject/WeakObjectPtrTemplates.h"

static constexpr float InfiniteDuration = FLT_MAX;

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitlesSubsystem)

void USubtitlesSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();

	BindDelegates();
}

void USubtitlesSubsystem::Deinitialize()
{
	UnbindDelegates();
}

void USubtitlesSubsystem::BindDelegates()
{
	if (TryCreateUMGWidget())
	{
		FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.BindUObject(this, &USubtitlesSubsystem::QueueSubtitle);
		FSubtitlesAndClosedCaptionsDelegates::QueueSubtitleFromSoundWaveName.BindUObject(this, &USubtitlesSubsystem::QueueSubtitleFromSoundWaveName);
		FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.BindUObject(this, &USubtitlesSubsystem::IsSubtitleActive);
		FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.BindUObject(this, &USubtitlesSubsystem::StopSubtitle);
		FSubtitlesAndClosedCaptionsDelegates::StopAllSubtitles.BindUObject(this, &USubtitlesSubsystem::StopAllSubtitles);
	}
}

void USubtitlesSubsystem::UnbindDelegates()
{
	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.Unbind();
	FSubtitlesAndClosedCaptionsDelegates::QueueSubtitleFromSoundWaveName.Unbind();
	FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.Unbind();
	FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.Unbind();
	FSubtitlesAndClosedCaptionsDelegates::StopAllSubtitles.Unbind();
}

void USubtitlesSubsystem::QueueSubtitle(FQueueSubtitleParameters Params, const ESubtitleTiming Timing)
{
	// Don't queue subtitles when the engine doesn't have them enabled or forced off. 
	if (GEngine == nullptr)
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Couldn't check if subtitles are enabled, because GEngine nullptr.");
		return;
	}
	if (!GEngine->bSubtitlesEnabled || GEngine->bSubtitlesForcedOff)
	{
		return;
	}

	const FSubtitleAssetData& Subtitle = Params.Subtitle;
	UE_LOGFMT(LogSubtitlesAndClosedCaptions, Display, "QueueSubtitle: {Name}", Subtitle.Text.ToString());

	// Externally-timed subtitles will be removed by the system queueing them, so they have an otherwise-infinite duration.
	float Duration = InfiniteDuration;
	if (Timing == ESubtitleTiming::InternallyTimed)
	{
		Duration = Params.Duration.IsSet() ? Params.Duration.GetValue() : Subtitle.Duration;
	}

	const float StartOffset = Params.StartOffset.IsSet() ? Params.StartOffset.GetValue() : Subtitle.StartOffset;

	if (IsInGameThread())
	{
		AddActiveSubtitle(Subtitle, Duration, StartOffset, Timing);
	}
	else
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesSubsystem::HandleQueueSubtitle"),
			[WeakThis = TWeakObjectPtr<USubtitlesSubsystem>(this), Subtitle, Duration, StartOffset, Timing]()
			{
				if (TStrongObjectPtr<USubtitlesSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->AddActiveSubtitle(Subtitle, Duration, StartOffset, Timing);
				}
			}
		);
	}
}

void USubtitlesSubsystem::QueueSubtitleFromSoundWaveName(const FName& SoundName, const FName& PackageName, const float Duration)
{
	// Move this work to the game thread.
	if (IsInGameThread())
	{
		FindAndQueueSubtitleFromSoundName(SoundName, PackageName, Duration);
	}
	else
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesSubsystem::FindAndQueueSubtitleFromSoundName"),
			[WeakThis = TWeakObjectPtr<USubtitlesSubsystem>(this), SoundName, PackageName, Duration]()
			{
				if (TStrongObjectPtr<USubtitlesSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->FindAndQueueSubtitleFromSoundName(SoundName, PackageName, Duration);
				}
			}
		);
	}
}

void USubtitlesSubsystem::AddActiveSubtitle(const FSubtitleAssetData& Subtitle, float Duration, const float StartOffset, const ESubtitleTiming Timing)
{
	if (!ensureMsgf(IsInGameThread(), TEXT("AddActiveSubtitle must be run on the GameThread. This should be done automatically through public functions like QueueSubtitle.")))
	{
		return;
	}

	// If the subtitle is already active then update its duration (by removing it then and then re-adding it)
	const FActiveSubtitleAssetData* FoundActiveSubtitle = ActiveSubtitles.FindByPredicate(
		[&Subtitle](const FActiveSubtitleAssetData& ActiveSubtitle) { return ActiveSubtitle.Subtitle == Subtitle; });
	if (FoundActiveSubtitle != nullptr)
	{
		RemoveActiveSubtitle(Subtitle);
	}

	FActiveSubtitleAssetData NewActiveSubtitle{ Subtitle };

	// Subtitles with delayed offset need a timer to await their entry in the queue.
	if (StartOffset > 0.f)
	{
		// The timer handle will be reused for the duration when it enters the active subtitle queue.
		// For now the timer tracks how long until it enters that queue.
		FTimerDelegate Delegate;
		Delegate.BindUFunction(this, FName(TEXT("MakeDelayedSubtitleActive")), NewActiveSubtitle.Subtitle, Timing);
		GetWorldRef().GetTimerManager().SetTimer(NewActiveSubtitle.DurationTimerHandle, MoveTemp(Delegate), StartOffset, /*bLoop=*/false);

		DelayedSubtitles.Add(MoveTemp(NewActiveSubtitle));
	}
	else
	{
		if (Timing == ESubtitleTiming::InternallyTimed)
		{
			// Without the delayed offset, instantly enter the queue as usual.
			// The timer here tracks how long until the subtitle will expire and leave the active subtitle queue.
			FTimerDelegate Delegate;
			Delegate.BindUFunction(this, FName(TEXT("RemoveActiveSubtitle")), NewActiveSubtitle.Subtitle);
			Duration = FMath::Max(Duration, SubtitleMinDuration);
			GetWorldRef().GetTimerManager().SetTimer(NewActiveSubtitle.DurationTimerHandle, MoveTemp(Delegate), Duration, /*bLoop=*/false);
		}
		
		AddAndDisplaySubtitle(NewActiveSubtitle);
	}
}

void USubtitlesSubsystem::FindAndQueueSubtitleFromSoundName(const FName& SoundName, const FName& PackageName, const float Duration)
{
	if (GEngine == nullptr || !GEngine->bSubtitlesEnabled || GEngine->bSubtitlesForcedOff)
	{
		return;
	}

	if (!ensure(AssetRegistry))
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Error, "Couldn't queue subtitles from {SoundName}; the AssetRegistry module wasn't loaded.", SoundName);
		return;
	}

	const FAssetData AssetData = AssetRegistry->GetAssetByObjectPath(FSoftObjectPath(FTopLevelAssetPath(PackageName, SoundName)));

	// These can't be const because HasAssetUserDataOfClass isn't
	// FastGetAsset does not load the asset. Don't bother loading if it isn't already.
	// It's already be loaded by the calling MSS, and this is called too often to do any blocking loads.
	UObject* SoundWaveAsset = AssetData.FastGetAsset(/* bLoad = false */);
	USoundWave* AsSoundWave = Cast<USoundWave>(SoundWaveAsset);

	if (IsValid(AsSoundWave) && AsSoundWave->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
	{
		// Using CastChecked because we already determined that the correct asset userdata exists.
		const UAssetUserData* UserData = AsSoundWave->GetAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());
		const USubtitleAssetUserData* SubtitleUserData = CastChecked<USubtitleAssetUserData>(UserData);

		for (const FSubtitleAssetData& Subtitle : SubtitleUserData->Subtitles)
		{
			FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(FQueueSubtitleParameters{ Subtitle, Duration }, ESubtitleTiming::InternallyTimed);
		}
	}
}

void USubtitlesSubsystem::MakeDelayedSubtitleActive(const FSubtitleAssetData& Subtitle, const ESubtitleTiming Timing)
{
	FActiveSubtitleAssetData* DelayedSubtitle = Algo::FindByPredicate(DelayedSubtitles, [&Subtitle](const FActiveSubtitleAssetData& DelayedSubtitle) {return DelayedSubtitle.Subtitle == Subtitle; });

	if (DelayedSubtitle != nullptr)
	{
		const FSubtitleAssetData& SubtitleAsset = DelayedSubtitle->Subtitle;
		const float Duration = FMath::Max(SubtitleAsset.Duration, SubtitleMinDuration);

		if (Timing == ESubtitleTiming::InternallyTimed)
		{
			// Reuse the Timer Handle for duration, now that it's no longer needed for the delay.
			FTimerDelegate Delegate;
			Delegate.BindUFunction(this, FName(TEXT("RemoveActiveSubtitle")), DelayedSubtitle->Subtitle);
			GetWorldRef().GetTimerManager().SetTimer(DelayedSubtitle->DurationTimerHandle, MoveTemp(Delegate), Duration, /*bLoop=*/false);
		}

		// Insert the new subtitle to the actual queue and ensure it remains sorted by priority.
		AddAndDisplaySubtitle(*DelayedSubtitle);

		// Remove from the list of Delayed Subtitles
		const int32 FirstRemovedIndex = Algo::RemoveIf(DelayedSubtitles, [&Subtitle](const FActiveSubtitleAssetData& DelayedSubtitle) {return DelayedSubtitle.Subtitle == Subtitle; });
		DelayedSubtitles.SetNum(FirstRemovedIndex);
	}
}

void USubtitlesSubsystem::AddAndDisplaySubtitle(FActiveSubtitleAssetData& NewActiveSubtitle)
{
	const FSubtitleAssetData& NewSubtitle = NewActiveSubtitle.Subtitle;
	const float NewPriority = NewSubtitle.Priority;
	const ESubtitleType Category = NewSubtitle.SubtitleType;

	bool ShouldAddSubtitle = true;
	// Warn if the subtitle won't play due to being lower-priority than the currently-playing subtitle in that category.
	for (const FActiveSubtitleAssetData& PreQueuedSubtitle : ActiveSubtitles)
	{
		const FSubtitleAssetData& PreQueuedSubtitleData = PreQueuedSubtitle.Subtitle;
		if (PreQueuedSubtitleData.SubtitleType == Category && (PreQueuedSubtitleData.Priority > NewPriority))
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Display, "Subtitle {Name} won't display: its priority is lower than the currently-playing subtitle in that category.", NewSubtitle.Text.ToString());
			ShouldAddSubtitle = false;
			break;
		}
	}

	// Remove newly lower-priority subtitles.
	FTimerManager& TimerManager = GetWorldRef().GetTimerManager();

	const int32 FirstRemovedIndex = Algo::StableRemoveIf(ActiveSubtitles,
		[NewPriority, Category](const FActiveSubtitleAssetData& RemovalCandidate) {return (RemovalCandidate.Subtitle.Priority < NewPriority) && RemovalCandidate.Subtitle.SubtitleType == Category; }
	);
	for (int32 Index = FirstRemovedIndex; Index < ActiveSubtitles.Num(); ++Index)
	{
		TimerManager.ClearTimer(ActiveSubtitles[Index].DurationTimerHandle);
	}

	ActiveSubtitles.SetNum(FirstRemovedIndex);

	if (ShouldAddSubtitle)
	{
		ActiveSubtitles.Add(MoveTemp(NewActiveSubtitle));

		// Sort by priority, but also prefer newer subtitles with a delayed start.
		ActiveSubtitles.StableSort([](const FActiveSubtitleAssetData& Lhs, const FActiveSubtitleAssetData& Rhs)
			{ return (Lhs.Subtitle.Priority > Rhs.Subtitle.Priority) || (Lhs.Subtitle.StartOffset > 0 && (Lhs.Subtitle.Priority == Rhs.Subtitle.Priority)); });

		UpdateWidgetData();
	}
}

bool USubtitlesSubsystem::IsSubtitleActive(FSubtitleAssetData Data) const
{
	// While other functions like StopSubtitle marshal the call onto the game thread, ::IsSubtitleActive can be called frequently via its delegate (eg, FMovieSceneSubtitlesSystemRunner::Evaluate), 
	// so marshaling each of these calls to the game thread if it's not there already is a perf hazard.
	if (!ensureMsgf(IsInGameThread(), TEXT("IsSubtitleActive must currently be run on the GameThread - ActiveSubtitles vector is not locked")))
	{
		return false;
	}

	const FActiveSubtitleAssetData* FoundActiveSubtitle = ActiveSubtitles.FindByPredicate(
		[Data](const FActiveSubtitleAssetData& ActiveSubtitle) { return ActiveSubtitle.Subtitle == Data; });

	return FoundActiveSubtitle != nullptr;
}

void USubtitlesSubsystem::StopSubtitle(FSubtitleAssetData Data)
{
	// While RemoveActiveSubtitle is called from some private functions,
	// StopSubtitle is the publicly-accessible entry point for it, and thus does an IsInGameThread check first.
	if (IsInGameThread())
	{
		RemoveActiveSubtitle(Data);
	}
	else
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesSubsystem::StopSubtitle"),
			[WeakThis = TWeakObjectPtr<USubtitlesSubsystem>(this), Data]()
			{
				if (TStrongObjectPtr<USubtitlesSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->RemoveActiveSubtitle(Data);
				}
			}
		);
	}
}

void USubtitlesSubsystem::StopAllSubtitles()
{
	if (!IsInGameThread())
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesSubsystem::StopAllSubtitles"),
			[WeakThis = TWeakObjectPtr<USubtitlesSubsystem>(this)]()
			{
				if (TStrongObjectPtr<USubtitlesSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->StopAllSubtitles();
				}
			}
		);
		return;
	}

	// Clean up queued subtitles.
	FTimerManager& TimerManager = GetWorldRef().GetTimerManager();
	for (FActiveSubtitleAssetData& ActiveSubtitle : ActiveSubtitles)
	{
		TimerManager.ClearTimer(ActiveSubtitle.DurationTimerHandle);
	}
	ActiveSubtitles.Empty();

	// Also remove delayed-start subtitles not yet in the queue.
	for (FActiveSubtitleAssetData& DelayedSubtitle : DelayedSubtitles)
	{
		TimerManager.ClearTimer(DelayedSubtitle.DurationTimerHandle);
	}
	DelayedSubtitles.Empty();

	// Clear the widget's display
	if (IsValid(ViewportWidget))
	{
		TWeakObjectPtr<USubtitleWidget> SubtitleWidget = GetActiveSubtitleWidget();
		if (SubtitleWidget.IsValid())
		{
			SubtitleWidget->StopDisplayingAllSubtitles();
		}
		else
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't remove subtitles because there isn't a valid UMG widget.");
		}
	}
	else
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't remove subtitles because the Viewport Widget couldn't be initialized.");
	}
}

void USubtitlesSubsystem::ReplaceWidget(const TSoftClassPtr<USubtitleWidget>& NewWidgetAsset)
{
	if (!IsInGameThread())
	{
		ExecuteOnGameThread(
			TEXT("USubtitlesSubsystem::ReplaceWidget"),
			[WeakThis = TWeakObjectPtr<USubtitlesSubsystem>(this), NewWidgetAsset]()
			{
				if (TStrongObjectPtr<USubtitlesSubsystem> ThisPin = WeakThis.Pin())
				{
					ThisPin->ReplaceWidget(NewWidgetAsset);
				}
			}
		);
		return;
	}

	if (!ActiveSubtitles.IsEmpty() || !DelayedSubtitles.IsEmpty())
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Error, "Can't replace the subtitle widget: there are still subtitles queued or displaying. Use StopAllSubtitles() first.");
		return;
	}

	if (!TryCreateUMGWidgetFromAsset(NewWidgetAsset))
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't replace the subtitle widget; was a valid asset provided?");
	}
}

void USubtitlesSubsystem::RemoveActiveSubtitle(const FSubtitleAssetData& Subtitle)
{
	if (!ensureMsgf(IsInGameThread(), TEXT("RemoveActiveSubtitle must be run on the GameThread - ActiveSubtitles vector is not locked")))
	{
		return;
	}

	bool bSuccessfullyRemoved = false;

	FTimerManager& TimerManager = GetWorldRef().GetTimerManager();

	// Remove currently-queued subtitles
	const int32 FirstRemovedIndex = Algo::StableRemoveIf(ActiveSubtitles, 
		[&Subtitle](const FActiveSubtitleAssetData& ActiveSubtitle) {return ActiveSubtitle.Subtitle == Subtitle; }
	);
	for (int32 Index = FirstRemovedIndex; Index < ActiveSubtitles.Num(); ++Index)
	{
		TimerManager.ClearTimer(ActiveSubtitles[Index].DurationTimerHandle);
		bSuccessfullyRemoved = true;
	}

	ActiveSubtitles.SetNum(FirstRemovedIndex);

	// Stop Displaying the removed subtitle and display a newly-most-relevant one if applicable.
	if (bSuccessfullyRemoved && IsValid(ViewportWidget))
	{
		TWeakObjectPtr<USubtitleWidget> SubtitleWidget = GetActiveSubtitleWidget();
		if (SubtitleWidget.IsValid())
		{
			SubtitleWidget->StopDisplayingSubtitle(Subtitle);

			if (!ActiveSubtitles.IsEmpty())
			{
				const FSubtitleAssetData& HighestPrioritySubtitle = ActiveSubtitles[0].Subtitle;
				SubtitleWidget->StartDisplayingSubtitle(HighestPrioritySubtitle);
			}
		}
		else
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't remove the next subtitle because there isn't a valid UMG widget.");
		}
	}
	else if (!IsValid(ViewportWidget))
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't remove subtitles because the Viewport Widget couldn't be initialized.");
	}

	// Remove delayed-start subtitles that haven't been queued yet that also use this asset.
	const int32 FirstRemovedActiveIndex = Algo::RemoveIf(DelayedSubtitles,
		[&Subtitle](const FActiveSubtitleAssetData& DelayedSubtitle) {return DelayedSubtitle.Subtitle == Subtitle; }
	);

	for (int32 Index = FirstRemovedActiveIndex; Index < DelayedSubtitles.Num(); ++Index)
	{
		TimerManager.ClearTimer(DelayedSubtitles[Index].DurationTimerHandle);
	}
	DelayedSubtitles.SetNum(FirstRemovedActiveIndex);
}

TWeakObjectPtr<USubtitleWidget> USubtitlesSubsystem::GetActiveSubtitleWidget() const
{
	if (IsValid(ViewportWidget))
	{
		return Cast<USubtitleWidget>(ViewportWidget->GetWidget());
	}
	return nullptr;
}

bool USubtitlesSubsystem::TryCreateUMGWidgetFromAsset(const TSoftClassPtr<USubtitleWidget>& WidgetToUse)
{
	if (IsValid(ViewportWidget))
	{

		// Synchronous load: called directly from ::Initialize when the World (and this subsystem) is created.
		WidgetToUse.LoadSynchronous();
		if (WidgetToUse.IsValid())
		{
			ViewportWidget->SetWidgetClass(WidgetToUse.Get());
			bInitializedWidget = false;	// The new widget will need to be displayed.
			return true;
		}
	}
	return false;
}

bool USubtitlesSubsystem::TryCreateUMGWidget()
{
	// Set up the UMG widget
	const USubtitlesSettings* Settings = GetDefault<USubtitlesSettings>();
	check(Settings != nullptr);
	const TSoftClassPtr<USubtitleWidget>& WidgetToUse = Settings->GetWidgetSoftClassPtr();

	if (!IsValid(ViewportWidget))
	{
		ViewportWidget = NewObject<UViewportWidgetOverlay>(this);
		if (!IsValid(ViewportWidget))
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Error, "Couldn't initialize the Subtitles Viewport Widget. Subtitles won't be displayed.");
			return false;
		}

		ViewportWidget->SetDisplayTypes(EViewportWidgetOverlay_DisplayType::Viewport, EViewportWidgetOverlay_DisplayType::Viewport, EViewportWidgetOverlay_DisplayType::Viewport);
	}

	if (!TryCreateUMGWidgetFromAsset(WidgetToUse))
	{
		// Fallback to default widget (not set by user):
		const TSoftClassPtr<USubtitleWidget>& WidgetToUseDefault = Settings->GetWidgetDefaultSoftClassPtr();
		if (WidgetToUseDefault.IsValid())
		{
			ViewportWidget->SetWidgetClass(WidgetToUseDefault.Get());
			// Log the fallback as a warning as seeing the default widget probably isn't intended.
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "A valid Subtitle Widget wasn't provided. The plugin's default widget will be used as a fallback.");
		}
		else
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Error, "The default Subtitle Widget asset isn't valid. Subtitles won't be displayed.");
		}
	}

	bInitializedWidget = false;
	return IsValid(ViewportWidget);
}

void USubtitlesSubsystem::UpdateWidgetData()
{
	// Update the widget. If it's not valid (eg, destroyed on non-seamless travel), try re-creating it first.
	if (IsValid(ViewportWidget) || (bInitializedWidget && TryCreateUMGWidget() && IsValid(ViewportWidget)))
	{
		if (!bInitializedWidget)
		{
			ViewportWidget->Display(GetWorld());
			bInitializedWidget = true;
		}

		TWeakObjectPtr<USubtitleWidget> SubtitleWidget = GetActiveSubtitleWidget();
		if (SubtitleWidget.IsValid())
		{
			check(SubtitleWidget->GetFlags() & RF_Transient);
			const FSubtitleAssetData& NewHighestPrioritySubtitle = ActiveSubtitles[0].Subtitle;
			SubtitleWidget->StartDisplayingSubtitle(NewHighestPrioritySubtitle);
		}
		else
		{
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't display subtitles, because there isn't a valid UMG Widget to display it to (check the Subtitle Widget and defaults in your project settings).");
		}
	}
	else
	{
		UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't display subtitles because there isn't a valid Viewport Widget to display it to.");
	}
}

