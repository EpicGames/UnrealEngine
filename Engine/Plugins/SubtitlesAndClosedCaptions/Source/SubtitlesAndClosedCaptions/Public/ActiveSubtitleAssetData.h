// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

struct FSubtitleAssetData;

#include "Engine/TimerHandle.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"
#include "UObject/ObjectPtr.h"

#include "ActiveSubtitleAssetData.generated.h"

USTRUCT()
struct FActiveSubtitleAssetData
{
	GENERATED_BODY()

	UPROPERTY()
	FSubtitleAssetData Subtitle;

	FTimerHandle DurationTimerHandle;
};
