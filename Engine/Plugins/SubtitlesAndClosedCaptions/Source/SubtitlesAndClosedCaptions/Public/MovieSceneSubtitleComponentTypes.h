// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneFwd.h"
#include "SubtitleDataComponent.h"

struct FSubtitleDataComponent;

class UMovieSceneSection;

// Component data present on all Subtitle entities
struct FMovieSceneSubtitleComponentTypes
{
	UE::MovieScene::TComponentTypeID<FSubtitleDataComponent> SubtitleData;

	SUBTITLESANDCLOSEDCAPTIONS_API static FMovieSceneSubtitleComponentTypes* Get();
	static void Destroy();

private:
	FMovieSceneSubtitleComponentTypes();

	static bool GMovieSceneSubtitlesComponentTypesDestroyed;
	static TUniquePtr<FMovieSceneSubtitleComponentTypes> GMovieSceneSubtitlesComponentTypes;
};
