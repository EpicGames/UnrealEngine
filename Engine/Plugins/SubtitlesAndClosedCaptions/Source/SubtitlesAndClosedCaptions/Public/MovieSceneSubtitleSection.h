// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneSubtitleSection.generated.h"

class USubtitleAssetUserData;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;

UCLASS(MinimalAPI)
class UMovieSceneSubtitleSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	TObjectPtr<const USubtitleAssetUserData> GetSubtitle() const
	{
		return Subtitle;
	}

	void SetSubtitle(const USubtitleAssetUserData& InSubtitle)
	{
		Subtitle = &InSubtitle;
	}

	//~ UMovieSceneSection interface
	SUBTITLESANDCLOSEDCAPTIONS_API virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;

	//~ IMovieSceneEntityProvider interface
	SUBTITLESANDCLOSEDCAPTIONS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:

	UPROPERTY(EditAnywhere, Category = "Subtitles")
	TObjectPtr<const USubtitleAssetUserData> Subtitle;
};
