// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/ObjectMacros.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneSubtitleSection.h"
#include "SubtitleDataComponent.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"

#include "MovieSceneSubtitlesSystem.generated.h"

struct FEvaluateSubtitles
{

	FEvaluateSubtitles(const UE::MovieScene::FInstanceRegistry& InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
	{
	}

	void ForEachAllocation(
		const UE::MovieScene::FEntityAllocation* Allocation,
		UE::MovieScene::TRead<UE::MovieScene::FInstanceHandle> SequenceInstanceHandles,
		UE::MovieScene::TWrite<FSubtitleDataComponent> SubtitleData) const
	{
		const UE::MovieScene::FBuiltInComponentTypes* BuiltInComponents = UE::MovieScene::FBuiltInComponentTypes::Get();

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			const UE::MovieScene::FInstanceHandle& SequenceInstanceHandle = SequenceInstanceHandles[Index];
			const UE::MovieScene::FSequenceInstance& SequenceInstance = InstanceRegistry.GetInstance(SequenceInstanceHandle);

			// Evaluate the subtitle associated with each subtitle section with logic dependent on changes in the context playing state
			Evaluate(SequenceInstance,
				SubtitleData[Index],
				SubtitleData[Index].LastSequenceInstanceStatus);

			SubtitleData[Index].LastSequenceInstanceStatus = SequenceInstance.GetContext().GetStatus();
		}
	}

	void Evaluate(const UE::MovieScene::FSequenceInstance& SequenceInstance, const FSubtitleDataComponent& SubtitleData, EMovieScenePlayerStatus::Type LastStatus) const
	{
		const FMovieSceneContext& Context = SequenceInstance.GetContext();

		const UMovieSceneSubtitleSection* SubtitleSection = SubtitleData.SubtitleSection;
		if (!ensureMsgf(SubtitleSection, TEXT("No valid subtitle section found in subtitles data!")))
		{
			return;
		}

		const USubtitleAssetUserData* SubtitleAsset = SubtitleData.SubtitleSection->GetSubtitle();
		if (!ensureMsgf(SubtitleAsset, TEXT("No valid subtitle found in subtitle section!")))
		{
			return;
		}

		const EMovieScenePlayerStatus::Type Status = Context.GetStatus();
#if !NO_LOGGING
		if (const UEnum* Enum = StaticEnum<EMovieScenePlayerStatus::Type>())
		{
			/* TODO: enable for ClangEditor Win64
			const FString LastStatusString = Enum->GetNameStringByIndex(LastStatus);
			const FString StatusString = Enum->GetNameStringByIndex(Status);
			UE_LOGFMT(LogSubtitlesAndClosedCaptions, Verbose, "LastStatus: {LastStatus}, Status: {CurStatus}, HasJumped: {Jumped}",
				LastStatusString, StatusString, Context.HasJumped());
			*/
		}
#endif // !NO_LOGGING

		// Stop all subtitles when we start playing to clear out any infinite duration subtitiles that were queued on pause/stop
		const bool bHasStarted{ (Status == EMovieScenePlayerStatus::Type::Playing || Status == EMovieScenePlayerStatus::Type::Scrubbing)
		&& (LastStatus == EMovieScenePlayerStatus::Type::Stopped || LastStatus == EMovieScenePlayerStatus::Type::Paused || LastStatus == EMovieScenePlayerStatus::Type::Stepping) };
		if (bHasStarted || Context.HasJumped())
		{
			FSubtitlesAndClosedCaptionsDelegates::StopAllSubtitles.ExecuteIfBound();
		}

		if (FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.IsBound())
		{
			const FFrameNumber LastSubtitleFrame = SubtitleSection->GetRange().GetUpperBoundValue();
			const FFrameNumber FirstSubtitleFrame = SubtitleSection->GetRange().GetLowerBoundValue();
			const FFrameNumber LastEvaluatedFrame = Context.GetFrameNumberRange().GetUpperBoundValue();

			// Only try to play subtitles if inside the section's first and last frames.
			if (LastEvaluatedFrame >= FirstSubtitleFrame && LastEvaluatedFrame < LastSubtitleFrame)
			{
				const FFrameRate FrameRate = Context.GetFrameRate();
				for (const FSubtitleAssetData& Entry : SubtitleAsset->Subtitles)
				{
					// The subtitle asset may have several subtitles.
					// To ensure each is queued at the correct time, check the start and end times of each against the current playback position.
					const FFrameTime StartOffset = Entry.StartOffset * FrameRate;
					const FFrameNumber FirstOffsetSubtitleFrame = FirstSubtitleFrame + StartOffset.FrameNumber;

					const FFrameTime LastFrameTime = Entry.Duration * FrameRate;
					const FFrameNumber LastOffsetSubtitleFrame = FirstOffsetSubtitleFrame + LastFrameTime.FrameNumber;

					if (LastEvaluatedFrame >= FirstOffsetSubtitleFrame && LastEvaluatedFrame < LastOffsetSubtitleFrame)
					{
						// Queue the subtitle to display in short (1/10th second) increments whenever playback is over the subtitle.
						// This allows for jumping and scrubbing the sequence without the subtitle getting stuck on or off.
						FQueueSubtitleParameters Params{ Entry };
						Params.Duration = 0.1f;
						Params.StartOffset = 0.f;

						FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle.ExecuteIfBound(Params, ESubtitleTiming::InternallyTimed);
					}
				}
			}
			else if(LastSubtitleFrame <= LastEvaluatedFrame)
			{
				for (const FSubtitleAssetData& Entry : SubtitleAsset->Subtitles)
				{
					if (FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive.Execute(Entry))
					{
						// If the subtitle is already active and outside the range of frames it should display for, remove it.
						FSubtitlesAndClosedCaptionsDelegates::StopSubtitle.ExecuteIfBound(Entry);
					}
				}
			}
		}
	}

protected:
	const UE::MovieScene::FInstanceRegistry& InstanceRegistry;
};

UCLASS()
class UMovieSceneSubtitlesSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:

	UMovieSceneSubtitlesSystem(const FObjectInitializer& ObjInit);
	virtual ~UMovieSceneSubtitlesSystem();

	//~ UMovieSceneEntitySystem members
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

