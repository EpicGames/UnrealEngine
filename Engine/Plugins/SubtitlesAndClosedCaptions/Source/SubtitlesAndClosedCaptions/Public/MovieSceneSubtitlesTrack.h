// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"

#include "MovieSceneSubtitlesTrack.generated.h"

class UMovieSceneSection;
class USubtitleAssetUserData;

UCLASS(MinimalAPI)
class UMovieSceneSubtitlesTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_UCLASS_BODY()

public:
	SUBTITLESANDCLOSEDCAPTIONS_API UMovieSceneSection* AddNewSubtitle(const USubtitleAssetUserData& Subtitle, FFrameNumber InStartTime);
	SUBTITLESANDCLOSEDCAPTIONS_API UMovieSceneSection* AddNewSubtitleOnRow(const USubtitleAssetUserData& Subtitle, FFrameNumber InStartTime, int32 InRowIndex);

public:

	// UMovieSceneTrack interface
	SUBTITLESANDCLOSEDCAPTIONS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual void RemoveAllAnimationData() override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual void AddSection(UMovieSceneSection& Section) override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual bool IsEmpty() const override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual bool SupportsMultipleRows() const override;
	SUBTITLESANDCLOSEDCAPTIONS_API virtual UMovieSceneSection* CreateNewSection() override;

private:
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> SubtitleSections;

};
