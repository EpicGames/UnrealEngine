// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneFwd.h"
#include "UObject/ObjectPtr.h"

#include "SubtitleDataComponent.generated.h"

class UMovieSceneSubtitleSection;

/** Component data for subtitles tracks */
USTRUCT()
struct FSubtitleDataComponent
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<const UMovieSceneSubtitleSection> SubtitleSection;

	EMovieScenePlayerStatus::Type LastSequenceInstanceStatus = EMovieScenePlayerStatus::Type::Stopped;
};
