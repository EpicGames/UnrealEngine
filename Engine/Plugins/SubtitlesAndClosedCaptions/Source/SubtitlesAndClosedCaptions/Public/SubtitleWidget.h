// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Blueprint/UserWidget.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

#include "SubtitleWidget.generated.h"

#define UE_API SUBTITLESANDCLOSEDCAPTIONS_API

class UBorder;
class UTextBlock;

UCLASS(Abstract, BlueprintType, Blueprintable, meta=( DontUseGenericSpawnObject="True", DisableNativeTick), MinimalAPI)
class USubtitleWidget : public UUserWidget
{
	GENERATED_BODY()

private:
	TMap<ESubtitleType, TOptional<FSubtitleAssetData>> CurrentlyDisplayedSubtitles;

protected:
	UE_API virtual void NativeConstruct() override;

	// Subtitle Text Blocks
	UPROPERTY(BlueprintReadOnly, Category = "Subtitles", meta = (BindWidgetOptional))
	TObjectPtr<UTextBlock> DialogSubtitleBlock;

	UPROPERTY(BlueprintReadOnly, Category = "Subtitles", meta = (BindWidgetOptional))
	TObjectPtr<UTextBlock> CaptionSubtitleBlock;

	UPROPERTY(BlueprintReadOnly, Category = "Subtitles", meta = (BindWidgetOptional))
	TObjectPtr<UTextBlock> DescriptionSubtitleBlock;

	// Borders to the text blocks
	UPROPERTY(BlueprintReadOnly, Category = "SubtitleBorders", meta = (BindWidgetOptional))
	TObjectPtr<UBorder> DialogBorder;

	UPROPERTY(BlueprintReadOnly, Category = "SubtitleBorders", meta = (BindWidgetOptional))
	TObjectPtr<UBorder> CaptionBorder;

	UPROPERTY(BlueprintReadOnly, Category = "SubtitleBorders", meta = (BindWidgetOptional))
	TObjectPtr<UBorder> DescriptionBorder;
public:
	void StartDisplayingSubtitle(const FSubtitleAssetData& Subtitle);
	void StopDisplayingSubtitle(const FSubtitleAssetData& Subtitle);
	void StopDisplayingAllSubtitles();

	UE_API USubtitleWidget(const FObjectInitializer& ObjectInitializer);
};

#undef UE_API