// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Subtitles/ISubtitlesAndClosedCaptionsModule.h"
#include "Templates/SubclassOf.h"

SUBTITLESANDCLOSEDCAPTIONS_API DECLARE_LOG_CATEGORY_EXTERN(LogSubtitlesAndClosedCaptions, Log, All);

class UAssetUserData;

class FSubtitlesAndClosedCaptionsModule : public ISubtitlesAndClosedCaptionsModule
{
public:
	void StartupModule() override;
	void ShutdownModule() override;
	TSubclassOf<UAssetUserData> GetAssetUserDataClass() const override;
};
