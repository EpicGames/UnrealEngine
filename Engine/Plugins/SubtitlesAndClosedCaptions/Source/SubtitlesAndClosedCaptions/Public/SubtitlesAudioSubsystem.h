// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Subsystems/AudioEngineSubsystem.h"
#include "ActiveSoundUpdateInterface.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"
#include "SubtitlesAudioSubsystem.generated.h"

#define UE_API SUBTITLESANDCLOSEDCAPTIONS_API

class UDialogueSoundWaveProxy;

/*
*	SubtitlesAudioSubsystem - AudioEngineSubsystem for automatically queueing subtitles on audio classes.
*
*	Separated from SubtitlesSubsystem, which is a UWorldSubsystem, to avoid issues with lifecycle management
*	as some parts need access to the world and others to the audio device.
*/
UCLASS(MinimalAPI, config = Game, defaultConfig)
class USubtitlesAudioSubsystem : public UAudioEngineSubsystem
	, public IActiveSoundUpdateInterface
{
	GENERATED_BODY()
public:
	USubtitlesAudioSubsystem() = default;

	//~ Begin USubsystem Interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem Interface

	//~ Begin IActiveSoundUpdateInterface
	virtual void NotifyActiveSoundCreated(FActiveSound& ActiveSound) override;
	virtual void NotifyActiveSoundDeleting(const FActiveSound& ActiveSound) override;
	//~ End IActiveSoundUpdateInterface

private:
	// Called on the audio thread when a SoundNodeDialoguePlayer first resolves its proxy for a given active sound.
	void OnDialogueSoundWaveProxyParsed(const FActiveSound& ActiveSound, const UDialogueSoundWaveProxy* Proxy);

	// Game-thread side of the same function
	void QueueSubtitleFromParsedSoundWaveProxy(const FText Text, const float Priority, const float Duration, const float StartOffset, const uint32 PlayOrder);

	static void QueueSubtitleFromDialogueWaveProxy(const UDialogueSoundWaveProxy* DialogueWaveProxy, const FActiveSound& ActiveSound);
	static void StopSubtitleFromDialogueWaveProxy(const UDialogueSoundWaveProxy* DialogueWaveProxy, const FActiveSound& ActiveSound);

	// Tracks which subtitle data is queued for each active sound. PlayOrder, the key, is monotonically incremented by FActiveSound and is thus unique.
	// Used later by NotifyActiveSoundDeleting for stopping the associated subtitles.
	TMap<uint32, TArray<FSubtitleAssetData>> QueuedSubtitlesByPlayOrder;
};

#undef UE_API
