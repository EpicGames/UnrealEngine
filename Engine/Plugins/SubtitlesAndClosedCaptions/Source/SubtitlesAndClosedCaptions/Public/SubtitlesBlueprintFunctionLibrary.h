// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"
#include "SubtitlesBlueprintFunctionLibrary.generated.h"

#define UE_API SUBTITLESANDCLOSEDCAPTIONS_API

class USubtitleWidget;

UCLASS(MinimalAPI, meta = (ScriptName = "SubtitlesLibrary"))
class USubtitlesBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Functions taking USubtitleAssetUserData as a parameter use TSoftObjectPtr to ensure that you can drag an asset onto BP variables feeding into them.
	// A const pointer allows drag/drop onto the function parameter itself, but TSoftObjectPtr is required for anything with the DefaultToInstanced specifier
	// (which in this case is the parent class UAssetUserData).

	// Queues all of the subtitles in a Subtitle asset, accounting for any delayed start offsets set for each. Subtitles queued through this function are internally-timed.
	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void QueueSubtitlesFromAsset(TSoftObjectPtr<USubtitleAssetUserData> SubtitleAsset);

	// Stops all subtitles in a subtitle asset, regardless of whether they're already queued or if they have a delayed start.
	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void StopSubtitlesInAsset(TSoftObjectPtr<USubtitleAssetUserData> SubtitleAsset);

	// Queues an individual subtitle from a struct, or a single entry in a Subtitle asset's array.
	UE_DEPRECATED(5.8, "Use QueueSingleSubtitle. It removes extraneous and confusing parameters.")
	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void QueueSubtitle(const FSubtitleAssetData& Subtitle, const ESubtitleTiming Timing = ESubtitleTiming::InternallyTimed);

	// Queues an individual subtitle.	
	// Parameters form an FSubtitleAssetData (see SubtitlesAndClosedCaptionsDelegates.h), but the Duration Type is removed as it is irrelevant for queueing this way.
	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void QueueSingleSubtitle(
		const FText Text, const float Duration, const float StartOffset, const float Priority, const ESubtitleType SubtitleType, const ESubtitleTiming Timing = ESubtitleTiming::InternallyTimed);


	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API bool IsSubtitleActive(const FSubtitleAssetData& Subtitle);

	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void StopSubtitle(const FSubtitleAssetData& Subtitle);

	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void StopAllSubtitles();

	UFUNCTION(BlueprintCallable, Category = Subtitles)
	static UE_API void ReplaceSubtitleWidget(const TSoftClassPtr<USubtitleWidget>& NewWidgetAsset);
};

#undef UE_API
