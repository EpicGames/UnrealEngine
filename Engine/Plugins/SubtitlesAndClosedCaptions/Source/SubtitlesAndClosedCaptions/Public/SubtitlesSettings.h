// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Math/Color.h"
#include "SubtitleWidget.h"
#include "SubtitlesSettings.generated.h"


UCLASS(config = Game, defaultconfig, meta = (DisplayName = "Subtitles And Closed Captions"))
class USubtitlesSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USubtitlesSettings();

	// Temporarily retaining the old GetWidget to not break builds.
	// Plugin is still experimental, so we don't need to actually deprecate it just yet.
	const TSubclassOf<USubtitleWidget>& GetWidget() const { return WidgetToUseAsSubclass; };
	const TSubclassOf<USubtitleWidget>& GetWidgetDefault() const { return DefaultWidgetToUseAsSubclass;	};
private:
	TSubclassOf<USubtitleWidget> WidgetToUseAsSubclass;
	TSubclassOf<USubtitleWidget> DefaultWidgetToUseAsSubclass;

public:
	inline const TSoftClassPtr<USubtitleWidget>& GetWidgetSoftClassPtr() const { return SubtitleWidgetToUse; }
	inline const TSoftClassPtr<USubtitleWidget>& GetWidgetDefaultSoftClassPtr() const { return SubtitleWidgetToUseDefault; }

protected:

	UPROPERTY(config, EditDefaultsOnly, Category = Subtitles, AdvancedDisplay)
	TSoftClassPtr<USubtitleWidget> SubtitleWidgetToUse;

	UPROPERTY(config)
	TSoftClassPtr<USubtitleWidget> SubtitleWidgetToUseDefault; // fallback for SubtitleWidgetToUse (not set by user)
};
