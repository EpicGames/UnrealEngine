// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ActiveSubtitleAssetData.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/Color.h"
#include "Subsystems/WorldSubsystem.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

#include "SubtitleWidget.h"
#include "ViewportWidgetOverlay.h"

#include "SubtitlesSubsystem.generated.h"

#define UE_API SUBTITLESANDCLOSEDCAPTIONS_API

class FCanvas;
class IAssetRegistry;
class UAssetUserData;
class UFont;
struct FQueueSubtitleParameters;

#if WITH_DEV_AUTOMATION_TESTS
namespace SubtitlesAndClosedCaptions::Test
{
#if 0 // Temporarily disabling these tests as they have a dangling reference that trips up the static analysis on certain build configurations.
	struct FMovieSceneSubtitlesTest;
#endif
	struct FSubtitlesTest;
}
#endif

/*
* #SUBTITLES_PRD -	Requirement:	Ability to allow designers to “script” subtitle location for sequences and scenes to avoid subtitles overlapping important scenes or characters
*					Use a UEngineSubsystem for blueprints
*
*	Game configuration for font customization per game
*/
UCLASS(MinimalAPI, config = Game, defaultConfig)
class USubtitlesSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	USubtitlesSubsystem() = default;

	// FSubtitlesAndClosedCaptionsDelegates
	// For thread safety, Queueing and Stopping subtitles will be marshaled onto the Game Thread. Their corresponding API calls will immediately return and be run asynchronously.

	// Adds a subtitle to the queue: Params contains the subtitle asset and an optional duration. The highest-priority subtitle in the queue will be displayed.
	// If Timing is ExternallyTimed, the queued subtitle will remain in the queue until manually removed.
	// If the subtitle asset has a non-zero StartOffset, it will sit in a delayed-start queue instead of being queued for display.
	UE_API virtual void QueueSubtitle(FQueueSubtitleParameters Params, const ESubtitleTiming Timing = ESubtitleTiming::InternallyTimed);

	// Metasounds may reference a soundwave, but can only really provide the sound's name from the audio thread.
	UE_API virtual void QueueSubtitleFromSoundWaveName(const FName& SoundName, const FName& PackageName, const float Duration);

	// Returns true if the given subtitle asset is being displayed.
	UE_API virtual bool IsSubtitleActive(FSubtitleAssetData Data) const;

	// Stops the given subtitle asset being displayed.  This includes subtitles not yet being displayed due to their StartOffset
	UE_API virtual void StopSubtitle(FSubtitleAssetData Data);

	// Stops all queued subtitles from being displayed.  This includes subtitles not yet being displayed due to their StartOffset.
	UE_API virtual void StopAllSubtitles();

	// Replaces the Subtitle Widget with a new asset subclassed from the base USubtitleWidget. Used by the Replace Subtitle Widget BP function.
	UE_API virtual void ReplaceWidget(const TSoftClassPtr<USubtitleWidget>& NewWidgetAsset);

protected:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void BindDelegates();
	UE_API virtual void UnbindDelegates();

	UPROPERTY()
	TMap<ESubtitleType, FSlateFontInfo> SubtitleFontInfo;

	// Sorted by priority, desc
	UPROPERTY()
	TArray<FActiveSubtitleAssetData> ActiveSubtitles;

	// Unsorted; subtitles with a delayed start offset still need to be tracked before entering the queue proper.
	UPROPERTY()
	TArray<FActiveSubtitleAssetData> DelayedSubtitles;

protected:
	UE_API virtual void AddActiveSubtitle(const FSubtitleAssetData& Subtitle, float Duration, const float StartOffset, const ESubtitleTiming Timing);

	UE_API virtual void FindAndQueueSubtitleFromSoundName(const FName& SoundName, const FName& PackageName, const float Duration);

	// These need to be UFUNCTIONs for the Timer Delegate bindings.
	UFUNCTION()
	UE_API virtual void MakeDelayedSubtitleActive(const FSubtitleAssetData& Subtitle, const ESubtitleTiming Timing);

	UFUNCTION()
	UE_API virtual void RemoveActiveSubtitle(const FSubtitleAssetData& Subtitle);

private:
	UE_API void AddAndDisplaySubtitle(FActiveSubtitleAssetData& NewActiveSubtitle);

public:
	UE_API TWeakObjectPtr<USubtitleWidget> GetActiveSubtitleWidget() const;

	// GetActiveSubtitles returns the sorted array of queued subtitles. Higher priorities are earlier in the array.
	// For equally-prioritized subtitles, the oldest one is first in this array.
	const TArray<FActiveSubtitleAssetData>& GetActiveSubtitles() const
	{
		return ActiveSubtitles;
	}

private:
	UPROPERTY(Transient)
	TObjectPtr<UViewportWidgetOverlay> ViewportWidget = nullptr;

	bool bInitializedWidget = false;
	IAssetRegistry* AssetRegistry = nullptr;
	UE_API bool TryCreateUMGWidgetFromAsset(const TSoftClassPtr<USubtitleWidget>& WidgetToUse);
	UE_API bool TryCreateUMGWidget();
	UE_API void UpdateWidgetData();
};

#undef UE_API
