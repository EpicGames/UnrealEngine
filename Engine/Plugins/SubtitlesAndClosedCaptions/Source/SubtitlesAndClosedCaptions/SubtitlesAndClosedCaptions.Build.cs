// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SubtitlesAndClosedCaptions : ModuleRules
	{
		public SubtitlesAndClosedCaptions(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Slate",
					"SlateCore",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"InputCore",
					"MovieSceneTracks",
					"MovieScene",
					"UMG",
					"ViewportWidgetOverlay"
				}
			);
		}
	}
}
