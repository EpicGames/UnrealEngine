// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleAssetActionUtility.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "BasicOverlays.h"
#include "EditorUtilityLibrary.h"
#include "IAssetTools.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

USubtitleAssetActionUtility::USubtitleAssetActionUtility()
{
	SupportedClasses.Add(UBasicOverlays::StaticClass());
}

// .SRT (SubRip Text) files are imported by the engine as UBasicOverlays, which contain an Array
// of captions with timestamps and positions. This utility creates a new asset that converts BasicOverlays to Subtitle assets.
void USubtitleAssetActionUtility::ConvertBasicOverlaysToSubtitles()
{
	TArray<UObject*> SelectedAssets = UEditorUtilityLibrary::GetSelectedAssets();

	for (UObject* Asset : SelectedAssets)
	{
		UBasicOverlays* OverlayAsset = Cast<UBasicOverlays>(Asset);
		if (IsValid(OverlayAsset))
		{
			const FString NameSuffix = TEXT("Subtitle");
			const FString AssetPath = FPaths::ProjectContentDir();
			FString AssetName = OverlayAsset->GetName() + NameSuffix;
			FString PackagePath = OverlayAsset->GetPackage()->GetPathName() + NameSuffix;

			UPackage* Package = CreatePackage(*PackagePath);
			USubtitleAssetUserData* NewSubtitleAsset = NewObject<USubtitleAssetUserData>(
				Package, *AssetName, EObjectFlags::RF_Public | EObjectFlags::RF_Standalone
			);

			// Iterate through each timecode in the overlay and create a new subtitle from it.
			for (const FOverlayItem& Overlay : OverlayAsset->GetAllOverlays())
			{
				FSubtitleAssetData Subtitle;
				Subtitle.Text = FText::FromString(Overlay.Text);
				
				// Timing
				Subtitle.StartOffset = Overlay.StartTime.GetTotalSeconds();
				Subtitle.Duration = Overlay.EndTime.GetTotalSeconds() - Overlay.StartTime.GetTotalSeconds();
				Subtitle.SubtitleDurationType = ESubtitleDurationType::UseDurationProperty;
				
				// Default priority, subtitle type, etc are fine.
				NewSubtitleAsset->Subtitles.Add(Subtitle);
			}

			FAssetRegistryModule::AssetCreated(NewSubtitleAsset);
			NewSubtitleAsset->MarkPackageDirty();
		}
	}
}