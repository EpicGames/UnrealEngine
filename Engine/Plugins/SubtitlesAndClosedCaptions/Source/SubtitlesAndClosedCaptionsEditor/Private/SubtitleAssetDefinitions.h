// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "SubtitlesAndClosedCaptionsEditorModule.h"

#include "SubtitleAssetDefinitions.generated.h"

UCLASS()
class USubtitleAssetDefinitions : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Subtitle", "Subtitle"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(97, 85, 212)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USubtitleAssetUserData::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_SubtitleSubMenu", "Subtitles"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
