// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubtitleFactory.h"

#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitleFactory)

USubtitleFactory::USubtitleFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	SupportedClass = USubtitleAssetUserData::StaticClass();
}

UObject* USubtitleFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext)
{
	return NewObject<USubtitleAssetUserData>(InParent, InClass, InName, InFlags);
}
