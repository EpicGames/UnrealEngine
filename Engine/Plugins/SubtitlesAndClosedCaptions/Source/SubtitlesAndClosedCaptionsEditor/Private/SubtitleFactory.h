// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "SubtitleFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class USubtitleFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory Begin
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext) override;
	// UFactory End
};
