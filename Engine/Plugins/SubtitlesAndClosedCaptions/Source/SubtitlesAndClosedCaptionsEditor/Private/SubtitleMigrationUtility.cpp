// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleMigrationUtility.h"

#include "EditorUtilityLibrary.h"
#include "Engine/Engine.h"
#include "Logging/StructuredLog.h"
#include "Sound/SoundWave.h"
#include "SubtitlesAndClosedCaptionsModule.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

const FString USubtitleMigrationUtility::UndoContext = TEXT("SubtitleMigration");

USubtitleMigrationUtility::USubtitleMigrationUtility()
{
	SupportedClasses.Add(USoundBase::StaticClass());
}

void USubtitleMigrationUtility::AddBlankSubtitle()
{
#if WITH_EDITOR
	if (GEngine)
	{
		// Setup transaction for undoing, and describe said action.
		const static FText Description = FText::FromString(TEXT("Add Blank Subtitle"));
		GEngine->BeginTransaction(*UndoContext, Description, nullptr);

		const TArray<UObject*> SelectedAssets = UEditorUtilityLibrary::GetSelectedAssetsOfClass(USoundBase::StaticClass());
		for (UObject* SelectedObject : SelectedAssets)
		{
			if (!IsValid(SelectedObject))
			{
				continue;
			}

			// GetSelectedAssetsOfClass and SupportedClasses should already only provide USoundBase
			USoundBase* SelectedSound = CastChecked<USoundBase>(SelectedObject);

			// Add an asset userdata if one doesn't already exist.
			if (!SelectedSound->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
			{
				SelectedSound->AddAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());
			}

			UAssetUserData* UserData = SelectedSound->GetAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());
			USubtitleAssetUserData* SubtitleData = CastChecked<USubtitleAssetUserData>(UserData);

			// Mark the userdata as modified for the undo action to work correctly.
			SubtitleData->Modify();

			SubtitleData->Subtitles.Add(FSubtitleAssetData());
		}

		// End transaction for undoing.
		GEngine->EndTransaction();
	}
#endif
}

void USubtitleMigrationUtility::RemoveLegacySubtitles()
{
#if WITH_EDITOR
	if (GEngine)
	{
		// Setup transaction for undoing, and describe said action.
		const static FText Description = FText::FromString(TEXT("Remove legacy subtitles from SoundWaves."));
		GEngine->BeginTransaction(*UndoContext, Description, nullptr);

		const TArray<UObject*> SelectedAssets = UEditorUtilityLibrary::GetSelectedAssetsOfClass(USoundWave::StaticClass());
		for (UObject* SelectedObject : SelectedAssets)
		{
			if (!IsValid(SelectedObject))
			{
				continue;
			}

			// This tool supports other non-wave Soundbases for other actions, but only USoundWaves have subtitles to migrate in this fashion.
			USoundWave* SelectedSound = Cast<USoundWave>(SelectedObject);

			// Only empty out the legacy subtitles if there's already new subtitles in place.
			if (IsValid(SelectedSound))
			{
				if (SelectedSound->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
				{
					SelectedSound->Modify();
					SelectedSound->Subtitles.Empty();
				}
				else
				{
					UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Can't remove legacy subtitles: migrate them to the new system first with Convert Legacy Subtitles.");
				}
			}
			else
			{
				UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Remove Legacy Subtitles can only be used on SoundWave assets.");
			}
		}

		GEngine->EndTransaction();
	}
#endif
}

void USubtitleMigrationUtility::ConvertLegacySubtitles()
{
#if WITH_EDITOR
	if (GEngine)
	{
		// Setup transaction for undoing, and describe said action.
		const static FText Description = FText::FromString(TEXT("Make Asset UserData from Legacy Subtitles"));
		GEngine->BeginTransaction(*UndoContext, Description, nullptr);

		const TArray<UObject*> SelectedAssets = UEditorUtilityLibrary::GetSelectedAssetsOfClass(USoundWave::StaticClass());
		for (UObject* SelectedObject : SelectedAssets)
		{
			if (!IsValid(SelectedObject))
			{
				continue;
			}

			// This tool supports other non-wave Soundbases for other actions, but only USoundWaves have subtitles to migrate in this fashion.
			USoundWave* SelectedSound = Cast<USoundWave>(SelectedObject);

			if (IsValid(SelectedSound))
			{
				// Add an asset userdata if one doesn't already exist.
				if (!SelectedSound->HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass()))
				{
					SelectedSound->AddAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());
				}

				UAssetUserData* UserData = SelectedSound->GetAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());
				USubtitleAssetUserData* SubtitleData = CastChecked<USubtitleAssetUserData>(UserData);

				// Loop through all the legacy subtitles and add a new subtitle to the UserData for each.
				for (const FSubtitleCue& Cue : SelectedSound->Subtitles)
				{
					FSubtitleAssetData Data;
					Data.SubtitleType = ESubtitleType::Subtitle;
					Data.SubtitleDurationType = ESubtitleDurationType::UseSoundDuration;

					Data.Priority = SelectedSound->SubtitlePriority;
					Data.Text = Cue.Text;
					Data.StartOffset = Cue.Time;
					Data.Comment = SelectedSound->Comment; // While it looks like this is the sound's comment, the property is in the Subtitles category.
					SubtitleData->Modify();
					SubtitleData->Subtitles.Add(Data);
				}
			}
			else
			{
				UE_LOGFMT(LogSubtitlesAndClosedCaptions, Warning, "Convert Legacy Subtitles can only be used on SoundWave assets.");
			}
		}

		GEngine->EndTransaction();
	}
#endif
}