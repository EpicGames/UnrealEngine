// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleSequencerSection.h"

#include "MovieSceneSection.h"
#include "MovieSceneSubtitleSection.h"
#include "SequencerSectionPainter.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"

FText FSubtitleSequencerSection::GetSectionTitle() const
{
	if (TStrongObjectPtr<const UMovieSceneSection> SectionPin = WeakSection.Pin())
	{
		const UMovieSceneSubtitleSection& SubtitlesSection = *CastChecked<const UMovieSceneSubtitleSection>(SectionPin.Get());
		if (const USubtitleAssetUserData* SubtitleAsset = SubtitlesSection.GetSubtitle())
		{
			const TArray<FSubtitleAssetData>& Subtitles = SubtitleAsset->Subtitles;
			if (!Subtitles.IsEmpty())
			{
				return Subtitles[0].Text;
			}
			else
			{
				return NSLOCTEXT("FSubtitlesSequencerSection", "EmptySubtitleAsset", "Empty Subtitle");
			}
		}
	}

	return NSLOCTEXT("FSubtitlesSequencerSection", "NoSubtitleName", "No Subtitle");
}

int32 FSubtitleSequencerSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();
	return LayerId;
}
