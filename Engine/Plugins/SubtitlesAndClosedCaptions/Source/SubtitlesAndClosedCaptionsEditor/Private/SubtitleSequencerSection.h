// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"

class FSequencerSectionPainter;

class FSubtitleSequencerSection
	: public FSequencerSection
	, public TSharedFromThis<FSubtitleSequencerSection>
{
public:
	FSubtitleSequencerSection(UMovieSceneSection& InSection)
		: FSequencerSection(InSection)
	{}

	// ISequencerSection interface
	virtual FText GetSectionTitle() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
};
