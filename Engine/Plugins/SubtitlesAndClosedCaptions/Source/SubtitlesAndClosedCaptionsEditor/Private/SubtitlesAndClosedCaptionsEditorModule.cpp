// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesAndClosedCaptionsEditorModule.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "SubtitlesTrackEditor.h"

#define LOCTEXT_NAMESPACE "SubtitlesAndClosedCaptionsEditor"

FText ISubtitlesAndClosedCaptionsEditorModule::AssetTypeCategory = LOCTEXT("SubtitlesAssetTypeCategory", "Data");

class FSubtitlesAndClosedCaptionsEditorModule : public ISubtitlesAndClosedCaptionsEditorModule
{
public:
	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked("SubtitlesAndClosedCaptions");

		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		CreateTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FSubtitlesTrackEditor::CreateTrackEditor));

	}

	virtual void ShutdownModule() override
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule != nullptr)
		{
			SequencerModule->UnRegisterTrackEditor(CreateTrackEditorHandle);
		}
	}

	FDelegateHandle CreateTrackEditorHandle;
};

IMPLEMENT_MODULE(FSubtitlesAndClosedCaptionsEditorModule, SubtitlesAndClosedCaptionsEditor);

#undef LOCTEXT_NAMESPACE
