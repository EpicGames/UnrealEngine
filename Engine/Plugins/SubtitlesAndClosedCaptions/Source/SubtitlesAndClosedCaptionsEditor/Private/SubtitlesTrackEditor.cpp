// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitlesTrackEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "LevelSequence.h"
#include "MovieSceneSubtitlesTrack.h"
#include "MovieSceneToolHelpers.h"
#include "MVVM/Views/ViewUtilities.h"
#include "SequencerSettings.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "SubtitleSequencerSection.h"


#define LOCTEXT_NAMESPACE "FSubtitlesTrackEditor"

FSubtitlesTrackEditor::FSubtitlesTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FSubtitlesTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShareable(new FSubtitlesTrackEditor(OwningSequencer));
}

FText FSubtitlesTrackEditor::GetDisplayName() const
{
	return LOCTEXT("SubtitilesTrackEditor_DisplayName", "Subtitles");
}

void FSubtitlesTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Subtitles Track"),
		LOCTEXT("AddTooltip", "Adds a new subtitles track that can display subtitles and closed captions."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Audio"), // #SUBTITLES_PRD_TODO - Add a subtitles track icon
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSubtitlesTrackEditor::HandleAddMenuEntryExecute)
		)
	);
}

void FSubtitlesTrackEditor::HandleAddMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddSubtitlesTrack_Transaction", "Add Subtitles Track"));
	FocusedMovieScene->Modify();

	UMovieSceneSubtitlesTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneSubtitlesTrack>();
	if (IsValid(NewTrack))
	{
		NewTrack->SetDisplayName(LOCTEXT("SubtitlesTrackName", "Subtitles"));

		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(NewTrack, FGuid());
		}
	}
}

bool FSubtitlesTrackEditor::IsDragDropValid(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams, DragDropValidInfo& OutDragDropInfo)
{
	if (!DragDropParams.Track.IsValid() || !DragDropParams.Track.Get()->IsA(UMovieSceneSubtitlesTrack::StaticClass()))
	{
		return false;
	}
	OutDragDropInfo.Operation = DragDropEvent.GetOperation();
	if (!OutDragDropInfo.Operation.IsValid() || !OutDragDropInfo.Operation->IsOfType<FAssetDragDropOp>())
	{
		return false;
	}
	OutDragDropInfo.SequencerPtr = GetSequencer();
	if (!OutDragDropInfo.SequencerPtr)
	{
		return false;
	}
	OutDragDropInfo.FocusedSequence = OutDragDropInfo.SequencerPtr->GetFocusedMovieSceneSequence();
	if (!OutDragDropInfo.FocusedSequence)
	{
		return false;
	}
	OutDragDropInfo.DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(OutDragDropInfo.Operation);

	return true;
}

TSharedPtr<SWidget> FSubtitlesTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	TDelegate OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FSubtitlesTrackEditor::OnAssetSelected, Track);
	TDelegate OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FSubtitlesTrackEditor::OnAssetEnterPressed, Track);
	return UE::Sequencer::MakeAddButton(
		LOCTEXT("SubtitleText", "Subtitle")
		, FOnGetContent::CreateSP(this, &FSubtitlesTrackEditor::BuildSubMenu, MoveTemp(OnAssetSelected), MoveTemp(OnAssetEnterPressed))
		, Params.ViewModel);
}

bool FSubtitlesTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (Asset->IsA<USubtitleAssetUserData>())
	{
		auto Subtitle = Cast<USubtitleAssetUserData>(Asset);
		UMovieSceneSubtitlesTrack* DummyTrack = nullptr;
		const FScopedTransaction Transaction(LOCTEXT("AddSubtitle_Transaction", "Add Subtitle"));
		if (TargetObjectGuid.IsValid())
		{
			TArray<TWeakObjectPtr<>> OutObjects;
			for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(TargetObjectGuid))
			{
				if (Object.IsValid())
				{
					OutObjects.Add(Object);
				}
			}
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubtitlesTrackEditor::AddNewAttachedSubtitle, Subtitle, DummyTrack, OutObjects));
		}
		else
		{
			int32 RowIndex = INDEX_NONE;
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubtitlesTrackEditor::AddNewSubtitle, Subtitle, DummyTrack, RowIndex));
		}
		return true;
	}
	return false;
}

TSharedRef<SWidget> FSubtitlesTrackEditor::BuildSubMenu(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FTopLevelAssetPath> ClassNames;
	ClassNames.Add(USubtitleAssetUserData::StaticClass()->GetClassPathName());
	TSet<FTopLevelAssetPath> DerivedClassNames;
	AssetRegistryModule.Get().GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

	FMenuBuilder MenuBuilder(true, nullptr);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = OnAssetSelected;
		AssetPickerConfig.OnAssetEnterPressed = OnAssetEnterPressed;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		for (FTopLevelAssetPath ClassName : DerivedClassNames)
		{
			AssetPickerConfig.Filter.ClassPaths.Add(ClassName);
		}
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);

	return MenuBuilder.MakeWidget();
}

void FSubtitlesTrackEditor::OnAssetSelected(const FAssetData& AssetData, UMovieSceneTrack* Track)
{
	FSlateApplication::Get().DismissAllMenus();

	const UObject* SelectedObject = AssetData.GetAsset();
	if (!SelectedObject)
	{
		return;
	}

	const USubtitleAssetUserData* NewAsset = CastChecked<const USubtitleAssetUserData>(AssetData.GetAsset());
	if (!NewAsset)
	{
		return;
	}
	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddSubtitle_Transaction", "Add Subtitle"));

	UMovieSceneSubtitlesTrack* SubtitlesTrack = Cast<UMovieSceneSubtitlesTrack>(Track);
	check(SubtitlesTrack);
	SubtitlesTrack->Modify();

	if (TSharedPtr<ISequencer> SequencerPin = GetSequencer())
	{
		UMovieSceneSection* NewSection = SubtitlesTrack->AddNewSubtitle(*NewAsset, SequencerPin->GetLocalTime().Time.FrameNumber);

		SequencerPin->EmptySelection();
		SequencerPin->SelectSection(NewSection);
		SequencerPin->ThrobSectionSelection();

		SequencerPin->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

void FSubtitlesTrackEditor::OnAssetEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* Track)
{
	if (!AssetData.IsEmpty())
	{
		OnAssetSelected(AssetData[0].GetAsset(), Track);
	}
}

TSharedRef<ISequencerSection> FSubtitlesTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShareable(new FSubtitleSequencerSection(SectionObject));
}

bool FSubtitlesTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneSubtitlesTrack::StaticClass();
}

bool FSubtitlesTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSubtitlesTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FSubtitlesTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	DragDropValidInfo DragDropInfo;
	if (!IsDragDropValid(DragDropEvent, DragDropParams, DragDropInfo))
	{
		return false;
	}
	for (const FAssetData& AssetData : DragDropInfo.DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(DragDropInfo.FocusedSequence, AssetData))
		{
			continue;
		}
		if (USubtitleAssetUserData* SubtitleAsset = Cast<USubtitleAssetUserData>(AssetData.GetAsset()))
		{
			FFrameRate TickResolution = DragDropInfo.SequencerPtr->GetFocusedTickResolution();
			FFrameNumber LengthInFrames = TickResolution.AsFrameNumber(SubtitleAsset->GetMaximumDuration());
			DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, DragDropParams.FrameNumber + LengthInFrames);
			return true;
		}
	}
	return false;
}

FReply FSubtitlesTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	DragDropValidInfo DragDropInfo;
	if (!IsDragDropValid(DragDropEvent, DragDropParams, DragDropInfo))
	{
		return FReply::Unhandled();
	}

	UMovieSceneSubtitlesTrack* SubtitlesTrack = Cast<UMovieSceneSubtitlesTrack>(DragDropParams.Track);

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropInfo.DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(DragDropInfo.FocusedSequence, AssetData))
		{
			continue;
		}

		USubtitleAssetUserData* SubtitleAsset = Cast<USubtitleAssetUserData>(AssetData.GetAsset());

		if (SubtitleAsset)
		{
			if (DragDropParams.TargetObjectGuid.IsValid())
			{
				TArray<TWeakObjectPtr<>> OutObjects;
				for (TWeakObjectPtr<> Object : DragDropInfo.SequencerPtr->FindObjectsInCurrentSequence(DragDropParams.TargetObjectGuid))
				{
					OutObjects.Add(Object);
				}

				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubtitlesTrackEditor::AddNewAttachedSubtitle, SubtitleAsset, SubtitlesTrack, OutObjects));
			}
			else
			{
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubtitlesTrackEditor::AddNewSubtitle, SubtitleAsset, SubtitlesTrack, DragDropParams.RowIndex));
			}

			bAnyDropped = true;
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}

/** Delegate for AnimatablePropertyChanged in HandleAssetAdded and OnDrop for adding subtitles.  */
FKeyPropertyResult FSubtitlesTrackEditor::AddNewSubtitle(FFrameNumber KeyTime, USubtitleAssetUserData* Subtitle, UMovieSceneSubtitlesTrack* InDestinationTrack, int32 RowIndex)
{
	FKeyPropertyResult KeyPropertyResult;

	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene->IsReadOnly())
	{
		return KeyPropertyResult;
	}

	// Undo tracking
	FocusedMovieScene->Modify();

	FFindOrCreateRootTrackResult<UMovieSceneSubtitlesTrack> TrackResult;
	TrackResult.Track = InDestinationTrack;

	// Make a new track if a valid one isn't provided.
	// ::HandleAssetAdded only passes a nullptr, but ::OnDrop should already have a valid destination track (if not, it's safe to create a new one).
	if (!InDestinationTrack)
	{
		TrackResult = FindOrCreateRootTrack<UMovieSceneSubtitlesTrack>();
		InDestinationTrack = TrackResult.Track;
	}

	if (ensure(InDestinationTrack))
	{
		InDestinationTrack->Modify();

		UMovieSceneSection* NewSection = InDestinationTrack->AddNewSubtitleOnRow(*Subtitle, KeyTime, RowIndex);

		// If a new track was created earlier, set display names and initialize it now.
		if (TrackResult.bWasCreated)
		{
			InDestinationTrack->SetDisplayName(LOCTEXT("SubtitleTrackName", "Subtitle"));

			if (GetSequencer().IsValid())
			{
				GetSequencer()->OnAddTrack(InDestinationTrack, FGuid());
			}
		}

		KeyPropertyResult.bTrackModified = true;
		KeyPropertyResult.SectionsCreated.Add(NewSection);
	}

	return KeyPropertyResult;
}

/** Delegate for AnimatablePropertyChanged in HandleAssetAdded and OnDrop for attached subtitles.  */
FKeyPropertyResult FSubtitlesTrackEditor::AddNewAttachedSubtitle(FFrameNumber KeyTime, class USubtitleAssetUserData* Subtitle, UMovieSceneSubtitlesTrack* InDestinationTrack, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo)
{
	FKeyPropertyResult KeyPropertyResult;

	for (int32 ObjectIndex = 0; ObjectIndex < ObjectsToAttachTo.Num(); ++ObjectIndex)
	{
		UObject* Object = ObjectsToAttachTo[ObjectIndex].Get();

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult;
			TrackResult.Track = InDestinationTrack;

			// Make a new track if a valid one isn't provided.
			// ::HandleAssetAdded only passes a nullptr, but ::OnDrop should already have a valid destination track (if not, it's safe to create a new one).
			if (!InDestinationTrack)
			{
				TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneSubtitlesTrack::StaticClass());
				InDestinationTrack = Cast<UMovieSceneSubtitlesTrack>(TrackResult.Track);
			}

			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(InDestinationTrack))
			{
				InDestinationTrack->Modify();

				UMovieSceneSection* NewSection = InDestinationTrack->AddNewSubtitle(*Subtitle, KeyTime);
				InDestinationTrack->SetDisplayName(LOCTEXT("SubtitleTrackName", "Subtitle"));
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);

				GetSequencer()->EmptySelection();
				GetSequencer()->SelectSection(NewSection);
				GetSequencer()->ThrobSectionSelection();
			}
		}
	}

	return KeyPropertyResult;
}
#undef LOCTEXT_NAMESPACE
