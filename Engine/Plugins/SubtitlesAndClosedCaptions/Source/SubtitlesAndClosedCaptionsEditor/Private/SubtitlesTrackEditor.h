// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

#include "ContentBrowserDelegates.h"

struct FGuid;
class FAssetDragDropOp;
class FReply;
class UMovieSceneTrack;
class UMovieSceneSubtitlesTrack;
class USubtitleAssetUserData;

struct DragDropValidInfo
{
	TSharedPtr<FDragDropOperation> Operation = nullptr;
	TSharedPtr<ISequencer> SequencerPtr = nullptr;
	UMovieSceneSequence* FocusedSequence = nullptr;
	TSharedPtr<FAssetDragDropOp> DragDropOp = nullptr;
};

/**
 * Tools for subtitles tracks
 */
class FSubtitlesTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FSubtitlesTrackEditor(TSharedRef<ISequencer> InSequencer);

	/**
	 * Creates an instance of this class.  Called by a sequencer
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;

protected:
	/** Delegate for AnimatablePropertyChanged in HandleAssetAdded for subtitles */
	FKeyPropertyResult AddNewSubtitle(FFrameNumber KeyTime, class USubtitleAssetUserData* Subtitle, UMovieSceneSubtitlesTrack* InDestinationTrack, int32 RowIndex);
	/** Delegate for AnimatablePropertyChanged in HandleAssetAdded for attached subtitles */
	FKeyPropertyResult AddNewAttachedSubtitle(FFrameNumber KeyTime, class USubtitleAssetUserData* Subtitle, UMovieSceneSubtitlesTrack* InDestinationTrack, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo);



private:
	TSharedRef<SWidget> BuildSubMenu(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed);
	void OnAssetSelected(const FAssetData& AssetData, UMovieSceneTrack* Track);
	void OnAssetEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* Track);
	void HandleAddMenuEntryExecute();

	// Helper function: packages up and null-checks common info used by OnAllowDrop and OnDrop
	bool IsDragDropValid(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams, DragDropValidInfo& OutDragDropInfo);
};
