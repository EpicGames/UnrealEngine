// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "AssetActionUtility.h"

#include "SubtitleAssetActionUtility.generated.h"

// Asset Action Utility class for any editor right-click actions relevant to subtitles.
UCLASS()
class USubtitleAssetActionUtility : public UAssetActionUtility
{
	GENERATED_BODY()

public:
	USubtitleAssetActionUtility();

	UFUNCTION(CallInEditor, Category="Subtitles")
	void ConvertBasicOverlaysToSubtitles();


};