// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "AssetActionUtility.h"

#include "SubtitleMigrationUtility.generated.h"

// Asset Action Utility class for migrating subtitles
// Associated asset SubtitleMigrationAction.uasset required to show up in editor.
UCLASS()
class USubtitleMigrationUtility : public UAssetActionUtility
{
	GENERATED_BODY()

private:
	const static FString UndoContext;

public:
	USubtitleMigrationUtility();

	UFUNCTION(CallInEditor, Category = "Subtitles")
	void AddBlankSubtitle();

	UFUNCTION(CallInEditor, Category = "Subtitles")
	void RemoveLegacySubtitles();

	UFUNCTION(CallInEditor, Category = "Subtitles")
	void ConvertLegacySubtitles();
};
