// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"

class ISubtitlesAndClosedCaptionsEditorModule : public IModuleInterface
{
public:
	UE_DEPRECATED(5.8, "Subtitle was moved over to the Data menu, use EAssetCategoryPaths::Data instead")
	static FText GetAssetTypeCategory()
	{
		return AssetTypeCategory;
	}
protected:
	static FText AssetTypeCategory;
};
