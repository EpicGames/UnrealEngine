// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SubtitlesAndClosedCaptionsEditor : ModuleRules
	{
		public SubtitlesAndClosedCaptionsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetTools",
					"Blutility",
					"LevelSequence",
					"MovieScene",
					"MovieSceneTools",
					"Overlay",
					"Sequencer",
					"SequencerCore",
					"Slate",
					"SlateCore",
					"SubtitlesAndClosedCaptions",
				}
			);

			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"Engine",
					"UnrealEd"
				}
			);
		}
	}
}
