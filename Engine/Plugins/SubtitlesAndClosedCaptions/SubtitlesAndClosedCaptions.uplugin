{
	"FileVersion": 3,
	"Version": 1000,
	"VersionName": "1.0.0.0",
	"FriendlyName": "Subtitles and Closed Captions",
	"Description": "Standalone plugin for displaying Subtitles and Closed Captions",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ViewportWidgetOverlay",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "SubtitlesAndClosedCaptions",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SubtitlesAndClosedCaptionsEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}
